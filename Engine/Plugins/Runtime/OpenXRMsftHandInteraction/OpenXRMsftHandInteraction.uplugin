{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenXRMsftHandInteraction",
	"Description": "OpenXRMsftHandInteraction provides support for the XR_MSFT_hand_interaction OpenXR Extension.  This allows hand tracking to act as a motion controller.",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "OpenXRMsftHandInteraction",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true
		}
	]
}