// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
    public class OpenXRMsftHandInteraction : ModuleRules
    {
        public OpenXRMsftHandInteraction(ReadOnlyTargetRules Target) 
				: base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
					"OpenXRHMD",
					"InputCore",
				}
				);

            AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenXR");
        }
    }
}
