// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRMsftHandInteraction.h"
#include "Modules/ModuleManager.h"
#include "Engine/Engine.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "OpenXRMsftHandInteraction"

IMPLEMENT_MODULE(FOpenXRMsftHandInteraction, OpenXRMsftHandInteraction);

namespace OpenXRMsftHandInteractionKeys
{
	const FKey OpenXRMsftHandInteraction_Left_Select("OpenXRMsftHandInteraction_Left_Select_Axis");
	const FKey OpenXRMsftHandInteraction_Right_Select("OpenXRMsftHandInteraction_Right_Select_Axis");

	const FKey OpenXRMsftHandInteraction_Right_Grip("OpenXRMsftHandInteraction_Right_Grip_Axis");
	const FKey OpenXRMsftHandInteraction_Left_Grip("OpenXRMsftHandInteraction_Left_Grip_Axis");
}

void FOpenXRMsftHandInteraction::StartupModule()
{
	RegisterOpenXRExtensionModularFeature();

	EKeys::AddMenuCategoryDisplayInfo("OpenXRMsftHandInteraction", LOCTEXT("OpenXRMsftHandInteractionSubCategory", "OpenXR Msft Hand Interaction"), TEXT("GraphEditor.PadEvent_16x"));

	EKeys::AddKey(FKeyDetails(OpenXRMsftHandInteractionKeys::OpenXRMsftHandInteraction_Left_Select,	LOCTEXT("OpenXRMsftHandInteraction_Left_Select_Axis", "OpenXRMsftHandInteraction (L) Select"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OpenXRMsftHandInteraction"));
	EKeys::AddKey(FKeyDetails(OpenXRMsftHandInteractionKeys::OpenXRMsftHandInteraction_Right_Select, LOCTEXT("OpenXRMsftHandInteraction_Right_Select_Axis", "OpenXRMsftHandInteraction (R) Select"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OpenXRMsftHandInteraction"));

	EKeys::AddKey(FKeyDetails(OpenXRMsftHandInteractionKeys::OpenXRMsftHandInteraction_Left_Grip, LOCTEXT("OpenXRMsftHandInteraction_Left_Grip_Axis", "OpenXRMsftHandInteraction (L) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OpenXRMsftHandInteraction"));
	EKeys::AddKey(FKeyDetails(OpenXRMsftHandInteractionKeys::OpenXRMsftHandInteraction_Right_Grip, LOCTEXT("OpenXRMsftHandInteraction_Right_Grip_Axis", "OpenXRMsftHandInteraction (R) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OpenXRMsftHandInteraction"));
}

bool FOpenXRMsftHandInteraction::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	OutExtensions.Add(XR_MSFT_HAND_INTERACTION_EXTENSION_NAME);
	return true;
}

bool FOpenXRMsftHandInteraction::GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics)
{
	OutKeyPrefixes.Add("OpenXRMsftHandInteraction");
	OutHasHaptics.Add(false);
	XrPath OutPath;
	XrResult Result = xrStringToPath(InInstance, "/interaction_profiles/microsoft/hand_interaction", &OutPath);
	OutPaths.Add(OutPath);
	return Result == XR_SUCCESS;
}

#undef LOCTEXT_NAMESPACE
