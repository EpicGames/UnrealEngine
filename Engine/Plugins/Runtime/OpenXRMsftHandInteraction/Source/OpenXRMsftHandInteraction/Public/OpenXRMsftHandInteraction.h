// Copyright Epic Games, Inc. All Rights Reserved.

/************************************************************************
 * This plugin is a good example of a very simple OpenXRExtension plugin.
 * 
 * It's all one class which implements both IOpenXRExtensionPlugin and
 * IModuleInterface.
 * 
 * If the XR_MSFT_hand_interaction OpenXR extension is available on the current
 * OpenXR runtime this OpenXRExtensionPlugin will register an Interaction Profile 
 * which makes the left/right hand poses available and will setup the Select and 
 * Grip buttons so they can be bound in UE4.
 ************************************************************************/

#pragma once
#include "IOpenXRExtensionPlugin.h"

#include "Modules/ModuleInterface.h"
#include "OpenXRCore.h"

class FOpenXRMsftHandInteraction : public IModuleInterface, public IOpenXRExtensionPlugin
{
public:
	/************************************************************************/
	/* IModuleInterface                                                     */
	/************************************************************************/
	virtual void StartupModule() override;

	/************************************************************************/
	/* IOpenXRExtensionPlugin                                               */
	/************************************************************************/
	virtual FString GetDisplayName() override
	{
		return FString(TEXT("OpenXRMsftHandInteraction"));
	}
	virtual bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual bool GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics) override;
};
